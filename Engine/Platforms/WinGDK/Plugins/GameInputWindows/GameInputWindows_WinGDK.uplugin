{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": [ "WinGDK" ],
	"Modules": [
		{
			"Name": "GameInputWindows",
			"Type": "Runtime",
			"PlatformAllowList": [
				"WinGDK"
			]
		}
	]
}