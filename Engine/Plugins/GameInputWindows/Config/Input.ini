[InputPlatformSettings_Windows InputPlatformSettings]
+HardwareDevices=(InputClassName="GameInput",HardwareDeviceIdentifier="Virtual",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=2055)
+HardwareDevices=(InputClassName="GameInput",HardwareDeviceIdentifier="Aggregate",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=4)
+HardwareDevices=(InputClassName="GameInput",HardwareDeviceIdentifier="XboxOne",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=260)
+HardwareDevices=(InputClassName="GameInput",HardwareDeviceIdentifier="Xbox360",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=260)
+HardwareDevices=(InputClassName="GameInput",HardwareDeviceIdentifier="Hid",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=0)
+HardwareDevices=(InputClassName="GameInput",HardwareDeviceIdentifier="I8042",PrimaryDeviceType=Unspecified,SupportedFeaturesMask=0)