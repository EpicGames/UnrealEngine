{
	"FileVersion": 3,
	"FriendlyName": "Game Input (Windows)",
	"Version": 1,
	"VersionName": "1.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "GameInput is a next-generation input API that exposes input devices of all kinds through a single consistent interface.",
	"Category": "Input Devices",
	"EnabledByDefault": false,
	"IsBetaVersion": true,
	"CanContainContent": false,
	"HasExplicitPlatforms": true,
	"bRequiresBuildPlatform": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "GameInputWindows",
			"Type": "RuntimeNoCommandlet",
			"HasExplicitPlatforms": true,
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "GameInput",
			"Enabled": true
		}
	]
}