// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool.Rules
{
	public class GameInputWindows : ModuleRules
	{
		public GameInputWindows(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
                    "ApplicationCore",
                    "Engine",
                    "InputCore",
                    "InputDevice",				
                    "CoreUObject",
                    "DeveloperSettings",
					"GameInputBase",
				}
			);
		}
	}
}