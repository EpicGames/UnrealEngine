// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputWindowsDevice.h"

#if GAME_INPUT_SUPPORT

#include "GameInputUtils.h"
#include "GameInputLogging.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleVariableRef
#include "GenericPlatform/IInputInterface.h"
#include "GameInputDeveloperSettings.h"
#include "GenericPlatform/InputDeviceRegistry.h"

FGameInputWindowsInputDevice::FGameInputWindowsInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, IGameInput* InGameInput)
	: IGameInputDeviceInterface(InMessageHandler, InGameInput)
{
#if WITH_EDITOR
	SetupEditorSettingListener();
#endif
}

FGameInputWindowsInputDevice::~FGameInputWindowsInputDevice()
{
#if WITH_EDITOR
	CleanupEditorSettingListener();
#endif
}

void FGameInputWindowsInputDevice::SetGameInputAndReinitialize(IGameInput* InGameInput)
{
	GameInput = InGameInput;
	Initialize();
}

GameInputKind FGameInputWindowsInputDevice::GetCurrentGameInputKindSupport() const
{
	GameInputKind RegisterInputKindMask = IGameInputDeviceInterface::GetCurrentGameInputKindSupport();

	// For now, we want to explicitly make sure that Keyboard and Mouse are NOT being processed by GameInput on windows.
	// This is because the WindowsApplication is doing to do all the processing of these types for us already, and 
	// having the Game Input device plugin process them too would result in "double" events and mouse accumulation.

#if !UE_BUILD_SHIPPING
	static bool bLogOnce = false;
	if (!bLogOnce)
	{
		if ((RegisterInputKindMask & GameInputKindKeyboard) != 0)
		{
			UE_LOGF(LogGameInput, Log, "[FGameInputWindowsInputDevice::GetCurrentGameInputKindSupport] Keyboard support was requested, but is not currently supported via the GameInput plugin on Windows.");
		}

		if ((RegisterInputKindMask & GameInputKindMouse) != 0)
		{
			UE_LOGF(LogGameInput, Log, "[FGameInputWindowsInputDevice::GetCurrentGameInputKindSupport] Mouse support was requested, but is not currently supported via the GameInput plugin on Windows.");
		}
		bLogOnce = true;
	}
#endif	// !UE_BUILD_SHIPPING

	// Clear the bits on these types to make sure we don't process any of them
	RegisterInputKindMask &= ~GameInputKindKeyboard;
	RegisterInputKindMask &= ~GameInputKindMouse;

	return RegisterInputKindMask;
}

void FGameInputWindowsInputDevice::HandleDeviceDisconnected(IGameInputDevice* Device, uint64 Timestamp)
{	
	if (Device)
	{
		if (FGameInputDeviceContainer* Data = GetDeviceData(Device))
		{
			// Clear any input state that might be related to this device
			Data->ClearInputState(GameInput);

			// Set it's device to nullptr because it is now disconnected
			Data->SetGameInputDevice(nullptr);
			UE_LOGF(LogGameInput, Log, "Game Input Device '%ls' Disconnected Successfully at Input Device ID '%d'", *UE::GameInput::LexToString(Device), Data->GetDeviceId().GetId());

			// Remap this device to the "unpaired" user because it has been disconnected
			const FPlatformUserId NewUserToAssign = IPlatformInputDeviceMapper::Get().GetUserForUnpairedInputDevices();
			const FInputDeviceId DeviceId = Data->GetDeviceId();

			if (DeviceId.IsValid())
			{
				const bool bSuccess = IPlatformInputDeviceMapper::Get().Internal_MapInputDeviceToUser(DeviceId, NewUserToAssign, EInputDeviceConnectionState::Disconnected);
				if (bSuccess)
				{
					Data->SetPlatformUserId(NewUserToAssign);
				}
			}			
		}
		else
		{
			UE_LOGF(LogGameInput, Error, "Game Input failed to disconnect a device. The Device '%ls' did not have an associated FGameInputWindowsInputDevice!", *UE::GameInput::LexToString(Device));
		}
	}
	else
	{
		UE_LOGF(LogGameInput, Warning, "Game Input failed to disconnect a device. The Device was null! %ls", *UE::GameInput::LexToString(Device));
	}

	EnumerateCurrentlyConnectedDeviceTypes();
}

void FGameInputWindowsInputDevice::HandleDeviceConnected(IGameInputDevice* Device, uint64 Timestamp)
{
	// get device information
	const GameInputDeviceInfo* Info = UE::GameInput::GetDeviceInfo(Device);
	UE_LOGF(LogGameInput, Log, "Game Input Device Connected: %ls  of kind: %ls", *UE::GameInput::LexToString(Device), *UE::GameInput::LexToString(Info->supportedInput));
	
	FGameInputDeviceContainer* Data = GetOrCreateDeviceData(Device);
	check(Data);

	// Map this input device to its user	
	const FInputDeviceId DeviceId = Data->GetDeviceId();
	
	// We only want to map this input device to a user if it has a valid Input Device Id.
	// Everything has an input device id, so if we got one that is invalid 
	// then that means this device has no processors in it because they were explicitly disabled.
	if (DeviceId.IsValid())
	{
		IPlatformInputDeviceMapper& DeviceMapper = IPlatformInputDeviceMapper::Get();
		
		const FPlatformUserId UserToAssign = DeviceMapper.GetPlatformUserForNewlyConnectedDevice();
		
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FInputDeviceScope InputScope(
			nullptr,
			UE::GameInput::GetDeviceScopeClassName(),
			DeviceMapper.GetUserIndexForPlatformUser(UserToAssign),
			FString(UE::GameInput::GetHardwareDeviceIdentifierName(Device)));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		{
			FInputDeviceDescriptor Desc;
			Desc.HardwareDeviceHandle     = DeviceId;
			Desc.InputDeviceName          = UE::GameInput::GetDeviceScopeClassName();
			Desc.HardwareDeviceIdentifier = FName(UE::GameInput::GetHardwareDeviceIdentifierName(Device));
			FInputDeviceRegistry::RegisterDevice(Desc);
		}
		const bool bSuccess = DeviceMapper.Internal_MapInputDeviceToUser(DeviceId, UserToAssign, EInputDeviceConnectionState::Connected);
		if (ensure(bSuccess))
		{
			Data->SetPlatformUserId(UserToAssign);
		}

		UE_LOGF(LogGameInput, Log, "Using PlatformUserId %d and InputDeviceId %d for device %ls",
			UserToAssign.GetInternalId(),
			DeviceId.GetId(),
			*UE::GameInput::LexToString(Device));
	}
	else
	{
		UE_LOGF(LogGameInput, Log, "Game Input Device %ls had no processors, so it will not be assiged to a platform user. You may need to configure it in the project settings.",
			*UE::GameInput::LexToString(Device));
	}
	
	EnumerateCurrentlyConnectedDeviceTypes();
}

FGameInputDeviceContainer* FGameInputWindowsInputDevice::CreateDeviceData(IGameInputDevice* InDevice)
{
	TSharedPtr<FGameInputDeviceContainer> Container = MakeShared<FGameInputDeviceContainer>(MessageHandler, InDevice, GetCurrentGameInputKindSupport());
	Container->InitalizeDeviceProcessors();

	// If this device has any processors assigned to it (meaning that it can send input events) then we want to assign it a new 
	// input device id. If there are no processors assigned to it, then it can't possibly send input events, so don't bother
	// mapping it to an input device ID. This can happen if the device is explicitly disallowed for the application 
	// via the project settings. In this case, we don't really want the rest of the engine to care about it at all
	const FInputDeviceId AssignedInputDeviceId = 
		Container->GetNumberOfProcessors() > 0 ?
		IPlatformInputDeviceMapper::Get().AllocateNewInputDeviceId() :
		INPUTDEVICEID_NONE;
	
	// This is a new device, we need to assign a new input device ID from the platform user
	Container->SetInputDeviceId(AssignedInputDeviceId);

	DeviceData.Emplace(Container);

	return Container.Get();
}

#if WITH_EDITOR

void FGameInputWindowsInputDevice::SetupEditorSettingListener()
{
	UGameInputDeveloperSettings* Settings = GetMutableDefault<UGameInputDeveloperSettings>();
	if (!Settings)
	{
		return;
	}

	EditorSettingChangedDelegate = Settings->OnInputSettingChanged.AddRaw(this, &FGameInputWindowsInputDevice::HandleEditorSettingChanged);	
}

void FGameInputWindowsInputDevice::CleanupEditorSettingListener()
{
	if (!EditorSettingChangedDelegate.IsValid())
	{
		return;
	}

	UGameInputDeveloperSettings* Settings = GetMutableDefault<UGameInputDeveloperSettings>();
	if (!Settings)
	{
		return;
	}
	
	Settings->OnInputSettingChanged.Remove(EditorSettingChangedDelegate);
}

void FGameInputWindowsInputDevice::HandleEditorSettingChanged()
{
	for (TSharedPtr<FGameInputDeviceContainer> Device : DeviceData)
	{
		GameInputKind SupportedKind = GetCurrentGameInputKindSupport();

		// Make sure that this device is still supported. If you are in the editor then you may have changed
		// this value and a connected device is no longer supported. If that is the case, then make sure we don't create
		// any new processors for it at all.
		IGameInputDevice* GameInputDevice = Device->GetGameInputDevice();
		if (!IsSupportedDevice(GameInputDevice))
		{
			UE_LOGF(LogGameInput, Log, "'%ls' is not supported (the settings have changed) and input will no longer be processed for it.", *UE::GameInput::LexToString(GameInputDevice));
			SupportedKind = GameInputKind::GameInputKindUnknown;
		}
		
		// Recreate the device processors for every device
		Device->RecreateDeviceProcessors(SupportedKind);

		// If this device previously had an invalid DeviceId because it had no processors, but now it does,
		// then we need to assign a FInputDeviceId because it can now process input
		// We will never change a device id to be invalid though, because that could cause weirdness where Slate
		// is expecting input events from a device which no longer exists.
		if (!Device->GetDeviceId().IsValid() && Device->GetNumberOfProcessors() > 0)
		{
			Device->SetInputDeviceId(IPlatformInputDeviceMapper::Get().AllocateNewInputDeviceId());
		}
	}
}
#endif	// WITH_EDITOR

#endif	//#if GAME_INPUT_SUPPORT