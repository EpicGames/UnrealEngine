// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if GAME_INPUT_SUPPORT

#include "IGameInputDeviceInterface.h"

class FGameInputWindowsInputDevice : public IGameInputDeviceInterface
{
public:
	explicit FGameInputWindowsInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler, struct IGameInput* InGameInput);
	virtual ~FGameInputWindowsInputDevice() override;

	void SetGameInputAndReinitialize(IGameInput* InGameInput);

protected:

	//~Begin IGameInputDeviceInterface interface	
	virtual GameInputKind GetCurrentGameInputKindSupport() const override;
	virtual void HandleDeviceDisconnected(IGameInputDevice* Device, uint64 Timestamp) override;
	virtual void HandleDeviceConnected(IGameInputDevice* Device, uint64 Timestamp) override;
	virtual FGameInputDeviceContainer* CreateDeviceData(IGameInputDevice* InDevice) override;
	//~End IGameInputDeviceInterface interface
	
private:
#if WITH_EDITOR
	void SetupEditorSettingListener();
	void CleanupEditorSettingListener();
	void HandleEditorSettingChanged();

	FDelegateHandle EditorSettingChangedDelegate;
#endif
};

#endif	// #if GAME_INPUT_SUPPORT