// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameInputWindowsModule.h"

#include "CoreGlobals.h"
#include "GameInputWindowsDevice.h"
#include "GameInputBaseModule.h"
#include "GameInputLogging.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "GameInputBaseIncludes.h"

namespace UE::GameInput
{
	/** The name of this modular feature plugin. */
	static const FName GameInputWindowsFeatureName = TEXT("GameInputWindows");
}

FGameInputWindowsModule& FGameInputWindowsModule::Get()
{
	return FModuleManager::LoadModuleChecked<FGameInputWindowsModule>(UE::GameInput::GameInputWindowsFeatureName);
}

bool FGameInputWindowsModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded(UE::GameInput::GameInputWindowsFeatureName);
}

TSharedPtr<IInputDevice> FGameInputWindowsModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
#if GAME_INPUT_SUPPORT
	//Get FGameInputBaseModule before anything else to make sure that it is loaded
	FGameInputBaseModule& BaseModule = FGameInputBaseModule::Get();
	IGameInput* BaseGameInputPtr = FGameInputBaseModule::GetGameInput();
	TSharedPtr<FGameInputWindowsInputDevice> DevicePtr = MakeShared<FGameInputWindowsInputDevice>(InMessageHandler, BaseGameInputPtr);
	
	if (BaseGameInputPtr)
	{
		DevicePtr->Initialize();
	}
	else
	{
		TWeakPtr<FGameInputWindowsInputDevice> WeakDevicePtr = DevicePtr;
		BaseModule.OnGameInputCreation.AddLambda([WeakDevicePtr](IGameInput* GameInputPtr)
		{
			if (TSharedPtr<FGameInputWindowsInputDevice> PinnedDevice = WeakDevicePtr.Pin())
			{
				PinnedDevice->SetGameInputAndReinitialize(GameInputPtr);
			}
		});
	}
	return DevicePtr;
	
#else
	UE_LOGF(LogGameInput, Error, "[FGameInputWindowsModule] Failed to create a GameInput device! GAME_INPUT_SUPPORT is false! See GameInputBase.build.cs and the build log for more info.");
	return nullptr;
#endif	// GAME_INPUT_SUPPORT
}

const TCHAR* FGameInputWindowsModule::GetPreferredDeviceAPIString() const
{
	static const TCHAR* APIName = TEXT("GameInput");
	return APIName;
}

IMPLEMENT_MODULE(FGameInputWindowsModule, GameInputWindows)
