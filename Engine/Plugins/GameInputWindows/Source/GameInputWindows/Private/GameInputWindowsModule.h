// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDeviceModule.h"

/**
 * Input Device module that will create the Game Input Device to handle input on Windows platforms.
 */
class FGameInputWindowsModule : public IInputDeviceModule
{
public:
	static FGameInputWindowsModule& Get();

	/** Returns true if this module is loaded (aka available) by the FModuleManager */
	static bool IsAvailable();

	//~ Begin IInputDeviceModule interface
	virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	virtual const TCHAR* GetPreferredDeviceAPIString() const override;
	//~ End IInputDeviceModule interface
};